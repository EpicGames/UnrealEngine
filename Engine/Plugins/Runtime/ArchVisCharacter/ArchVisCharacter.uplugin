{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "ArchVis Character",
	"Description" : "A controllable character tuned for architectural applications",
	"Category" : "Gameplay",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "ArchVisCharacter",
			"Type" : "Runtime",
			"LoadingPhase" : "Default"
		}
	]
}