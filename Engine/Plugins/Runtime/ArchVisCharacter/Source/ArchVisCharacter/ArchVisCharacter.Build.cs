// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ArchVisCharacter : ModuleRules
	{
		public ArchVisCharacter(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
                    "Engine",
//                     "RenderCore",
//                     "RHI"
				}
				);
		}
	}
}
