// Copyright Epic Games, Inc. All Rights Reserved. 

#include "ArchVisCharMovementComponent.h"
#include "GameFramework/Character.h"
#include "GameFramework/Controller.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ArchVisCharMovementComponent)

UArchVisCharMovementComponent::UArchVisCharMovementComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	RotationalAcceleration = FRotator(300.f, 300.f, 0.f);
	RotationalDeceleration = FRotator(300.f, 300.f, 0.f);
	MaxRotationalVelocity = FRotator(80.f, 100.f, 0.f);
	MinPitch = -85.f;
	MaxPitch = 85.f;
	
	WalkingFriction = 4.f;
	WalkingSpeed = 165.f;
	WalkingAcceleration = 500.f;

	BrakingDecelerationWalking = 0.f;
}

void UArchVisCharMovementComponent::OnRegister()
{
	Super::OnRegister();

	// copy our simplified params to the corresponding real params
	GroundFriction = WalkingFriction;
	MaxWalkSpeed = WalkingSpeed;
	MaxAcceleration = WalkingAcceleration;
}

void UArchVisCharMovementComponent::PhysWalking(float DeltaTime, int32 Iterations)
{
	// let character do its thing for translation
	Super::PhysWalking(DeltaTime, Iterations);

	//
	// now we will handle rotation
	//

	// update yaw
	if (CurrentRotInput.Yaw == 0)
	{
		// decelerate to 0
		if (CurrentRotationalVelocity.Yaw > 0.f)
		{
			CurrentRotationalVelocity.Yaw -= RotationalDeceleration.Yaw * DeltaTime;
			CurrentRotationalVelocity.Yaw = FMath::Max(CurrentRotationalVelocity.Yaw, 0.f);		// don't go below 0 because that would be re-accelerating the other way
		}
		else
		{
			CurrentRotationalVelocity.Yaw += RotationalDeceleration.Yaw * DeltaTime;
			CurrentRotationalVelocity.Yaw = FMath::Min(CurrentRotationalVelocity.Yaw, 0.f);		// don't go above 0 because that would be re-accelerating the other way
		}
	}
	else
	{
		// accelerate in desired direction
		// clamp delta so it won't take us outside the acceptable speed range
		// note that if we're already out of that range, we won't clamp back
		float const MaxYawVelMag = FMath::Min(1.f, FMath::Abs(CurrentRotInput.Yaw)) * MaxRotationalVelocity.Yaw;
		float const MaxDeltaYawVel = FMath::Max(0.f, MaxYawVelMag - CurrentRotationalVelocity.Yaw);
		float const MinDeltaYawVel = FMath::Min(0.f, -(CurrentRotationalVelocity.Yaw + MaxYawVelMag));
		float const DeltaYaw = FMath::Clamp(CurrentRotInput.Yaw * RotationalAcceleration.Yaw * DeltaTime, MinDeltaYawVel, MaxDeltaYawVel);
		CurrentRotationalVelocity.Yaw += DeltaYaw;
	}

	// update pitch
	if (CurrentRotInput.Pitch == 0)
	{
		// decelerate to 0
		if (CurrentRotationalVelocity.Pitch > 0.f)
		{
			CurrentRotationalVelocity.Pitch -= RotationalDeceleration.Pitch * DeltaTime;
			CurrentRotationalVelocity.Pitch = FMath::Max(CurrentRotationalVelocity.Pitch, 0.f);		// don't go below 0 because that would be reaccelerating the other way
		}
		else
		{
			CurrentRotationalVelocity.Pitch += RotationalDeceleration.Pitch * DeltaTime;
			CurrentRotationalVelocity.Pitch = FMath::Min(CurrentRotationalVelocity.Pitch, 0.f);		// don't go above 0 because that would be reaccelerating the other way
		}
	}
	else
	{
		float const MaxPitchVelMag = FMath::Min(1.f, FMath::Abs(CurrentRotInput.Pitch)) * MaxRotationalVelocity.Pitch;
		float const MaxDeltaPitchVel = FMath::Max(0.f, MaxPitchVelMag - CurrentRotationalVelocity.Pitch);
		float const MinDeltaPitchVel = FMath::Min(0.f, -(CurrentRotationalVelocity.Pitch + MaxPitchVelMag));
		float const DeltaPitch = FMath::Clamp(CurrentRotInput.Pitch * RotationalAcceleration.Pitch * DeltaTime, MinDeltaPitchVel, MaxDeltaPitchVel);
		CurrentRotationalVelocity.Pitch += DeltaPitch;
	}

	// apply rotation
	FRotator RotDelta = CurrentRotationalVelocity * DeltaTime;
	if (!RotDelta.IsNearlyZero() && CharacterOwner)
	{
		FRotator const ViewRot = CharacterOwner->GetControlRotation();
		FRotator const ComponentRot = UpdatedComponent->GetComponentRotation();
		FRotator const CurrentComponentRot(ViewRot.Pitch, ComponentRot.Yaw, ComponentRot.Roll);

		// enforce pitch limits
		float const CurrentPitch = CurrentComponentRot.Pitch;
		float const MinDeltaPitch = MinPitch - CurrentPitch;
		float const MaxDeltaPitch = MaxPitch - CurrentPitch;
		float const OldPitch = RotDelta.Pitch;
		RotDelta.Pitch = FMath::Clamp(RotDelta.Pitch, MinDeltaPitch, MaxDeltaPitch);
		if (OldPitch != RotDelta.Pitch)
		{
			// if we got clamped, zero the pitch velocity
			CurrentRotationalVelocity.Pitch = 0.f;
		}

		FRotator const NewRot = CurrentComponentRot + RotDelta;

		FHitResult Hit(1.f);
		SafeMoveUpdatedComponent(FVector::ZeroVector, FRotator(0.f, NewRot.Yaw, NewRot.Roll), false, Hit);
		CharacterOwner->GetController()->SetControlRotation(FRotator(NewRot.Pitch, 0.f, 0.f));

		// TODO: Do Cosine scaling?
		float const PercentSpeed = (90.f - fabs(NewRot.Pitch)) / 90.f;
		MaxWalkSpeed = WalkingSpeed * PercentSpeed;
		MaxAcceleration = WalkingAcceleration * PercentSpeed;
	}

	// consume input
	CurrentRotInput = FRotator::ZeroRotator;
}

void UArchVisCharMovementComponent::AddRotInput(float Pitch, float Yaw, float Roll)
{
	CurrentRotInput.Roll += Roll;
	CurrentRotInput.Pitch += Pitch;
	CurrentRotInput.Yaw += Yaw;
}
