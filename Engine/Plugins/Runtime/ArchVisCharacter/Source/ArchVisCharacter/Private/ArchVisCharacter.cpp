// Copyright Epic Games, Inc. All Rights Reserved.

#include "ArchVisCharacter.h"
#include "Components/InputComponent.h"
#include "ArchVisCharMovementComponent.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ArchVisCharacter)

AArchVisCharacter::AArchVisCharacter(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer.SetDefaultSubobjectClass<UArchVisCharMovementComponent>(ACharacter::CharacterMovementComponentName))
{
	TurnAxisName = TEXT("Turn");
	TurnAtRateAxisName = TEXT("TurnRate");
	LookUpAxisName = TEXT("LookUp");
	LookUpAtRateAxisName = TEXT("LookUpRate");
	MoveForwardAxisName = TEXT("MoveForward");
	MoveRightAxisName = TEXT("MoveRight");

	bUseControllerRotationPitch = false;
	bUseControllerRotationYaw = false;
	bUseControllerRotationRoll = false;

	MouseSensitivityScale_Pitch = 0.025f;
	MouseSensitivityScale_Yaw = 0.025f;
}

UArchVisCharMovementComponent* AArchVisCharacter::GetArchVisCharMoveComponent() const
{
	return Cast<UArchVisCharMovementComponent>(GetMovementComponent());
}

void AArchVisCharacter::SetupPlayerInputComponent(UInputComponent* PlayerInputComponent)
{
	Super::SetupPlayerInputComponent(PlayerInputComponent);

	PlayerInputComponent->BindAxis(*TurnAxisName, this, &AArchVisCharacter::Turn);
	PlayerInputComponent->BindAxis(*LookUpAxisName, this, &AArchVisCharacter::LookUp);
	PlayerInputComponent->BindAxis(*TurnAtRateAxisName, this, &AArchVisCharacter::TurnAtRate);
	PlayerInputComponent->BindAxis(*LookUpAtRateAxisName, this, &AArchVisCharacter::LookUpAtRate);

	PlayerInputComponent->BindAxis(*MoveForwardAxisName, this, &AArchVisCharacter::MoveForward);
	PlayerInputComponent->BindAxis(*MoveRightAxisName, this, &AArchVisCharacter::MoveRight);
}

void AArchVisCharacter::Turn(float Val)
{
	// experimental
	if (Val != 0.f)
	{
		UArchVisCharMovementComponent* const MoveComp = GetArchVisCharMoveComponent();
		if (MoveComp)
		{
			// so direct inputs like mouse movements are framerate-independent
			float const InputVelocity = Val / GetWorld()->GetDeltaSeconds();

			float const ClampedVal = FMath::Clamp(MouseSensitivityScale_Yaw * InputVelocity, -1.f, 1.f);
			MoveComp->AddRotInput(0.f, ClampedVal, 0.f);
		}
	}
}

void AArchVisCharacter::TurnAtRate(float Val)
{
	if (Val != 0.f)
	{
		UArchVisCharMovementComponent* const MoveComp = GetArchVisCharMoveComponent();
		if (MoveComp)
		{
			MoveComp->AddRotInput(0.f, Val, 0.f);
		}
	}
}

void AArchVisCharacter::LookUp(float Val)
{
	// experimental
	if (Val != 0.f)
	{
		UArchVisCharMovementComponent* const MoveComp = GetArchVisCharMoveComponent();
		if (MoveComp)
		{
			// so direct inputs like mouse movements are framerate-independent
			float const InputVelocity = Val / GetWorld()->GetDeltaSeconds();
				
			float const ClampedVal = FMath::Clamp(MouseSensitivityScale_Pitch * InputVelocity, -1.f, 1.f);
			MoveComp->AddRotInput(-ClampedVal, 0.f, 0.f);
		}
	}
}

void AArchVisCharacter::LookUpAtRate(float Val)
{
	if (Val != 0.f)
	{
		UArchVisCharMovementComponent* const MoveComp = GetArchVisCharMoveComponent();
		if (MoveComp)
		{
			MoveComp->AddRotInput(-Val, 0.f, 0.f);
		}
	}
}

void AArchVisCharacter::MoveRight(float Val)
{
	if (Val != 0.f)
	{
		if (GetController())
		{
			FRotator const ControlSpaceRot = GetActorRotation();

			// transform to world space and add it
			AddMovementInput(FRotationMatrix(ControlSpaceRot).GetScaledAxis(EAxis::Y), Val);
		}
	}
}

void AArchVisCharacter::MoveForward(float Val)
{
	if (Val != 0.f)
	{
		if (GetController())
		{
			FRotator const ControlSpaceRot = GetActorRotation();

			// transform to world space and add it
			AddMovementInput(FRotationMatrix(ControlSpaceRot).GetScaledAxis(EAxis::X), Val);
		}
	}
}

FRotator AArchVisCharacter::GetViewRotation() const
{
	// pawn rotation dictates camera rotation
	float const Pitch = GetControlRotation().Pitch;
	float const Yaw = GetActorRotation().Yaw;
	return FRotator(Pitch, Yaw, 0.f);
}

