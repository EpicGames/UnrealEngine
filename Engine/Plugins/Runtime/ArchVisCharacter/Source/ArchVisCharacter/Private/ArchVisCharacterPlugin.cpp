// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"


class FArchVisCharacterPlugin : public IModuleInterface
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

IMPLEMENT_MODULE(FArchVisCharacterPlugin, ArchVisCharacter)


void FArchVisCharacterPlugin::StartupModule()
{
	
}

void FArchVisCharacterPlugin::ShutdownModule()
{
	
}



