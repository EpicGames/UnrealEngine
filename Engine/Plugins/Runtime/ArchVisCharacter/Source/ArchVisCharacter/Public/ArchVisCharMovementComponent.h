// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/CharacterMovementComponent.h"
#include "ArchVisCharMovementComponent.generated.h"

#define UE_API ARCHVISCHARACTER_API

UCLASS(MinimalAPI)
class UArchVisCharMovementComponent : public UCharacterMovementComponent
{
	GENERATED_BODY()

public:
	/** Ctor */
	UE_API UArchVisCharMovementComponent(const FObjectInitializer& ObjectInitializer);

	/** Controls how fast the character's turn rate accelerates when rotating and looking up/down */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "ArchVis Controls")
	FRotator RotationalAcceleration;

	/** Controls how fast the character's turn rate decelerates to 0 when user stops turning */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "ArchVis Controls")
	FRotator RotationalDeceleration;

	/** Fastest possible turn rate */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "ArchVis Controls")
	FRotator MaxRotationalVelocity;

	/** Controls how far down you can look */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "ArchVis Controls")
	float MinPitch;

	/** Controls how far up you can look */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "ArchVis Controls")
	float MaxPitch;

	/** Controls walking deceleration. */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "ArchVis Controls")
	float WalkingFriction;

	/** How fast the character can walk. */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "ArchVis Controls")
	float WalkingSpeed;

	/** How fast the character accelerates. */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "ArchVis Controls")
	float WalkingAcceleration;

public:
	/** Adds rotational input. */
	UE_API void AddRotInput(float Pitch, float Yaw, float Roll);

protected:

	FRotator CurrentRotationalVelocity;

	FRotator CurrentRotInput;

	UE_API virtual void PhysWalking(float DeltaTime, int32 Iterations) override;
	UE_API virtual void OnRegister() override;
};


#undef UE_API
