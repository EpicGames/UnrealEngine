// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Character.h"
#include "ArchVisCharacter.generated.h"

#define UE_API ARCHVISCHARACTER_API

class UInputComponent;

UCLASS(MinimalAPI, Blueprintable)
class AArchVisCharacter : public ACharacter
{
	GENERATED_BODY()

public:
	/** Ctor */
	UE_API AArchVisCharacter(const FObjectInitializer& ObjectInitializer);

	/** Axis name for direct look up/down inputs (e.g. mouse). This should match an Axis Binding in your input settings */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "ArchVis Controls")
	FString LookUpAxisName;

	/** Axis name for rate-based look up/down inputs (e.g. joystick). This should match an Axis Binding in your input settings */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "ArchVis Controls")
	FString LookUpAtRateAxisName;

	/** Axis name for direct turn left/right inputs (e.g. mouse). This should match an Axis Binding in your input settings */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "ArchVis Controls")
	FString TurnAxisName;

	/** Axis name for rate-based turn left/right inputs (e.g. joystick). This should match an Axis Binding in your input settings */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "ArchVis Controls")
	FString TurnAtRateAxisName;

	/** Axis name for "move forward/back" control. This should match an Axis Binding in your input settings */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "ArchVis Controls")
	FString MoveForwardAxisName;
	
	/** Axis name for "move left/right" control. This should match an Axis Binding in your input settings */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "ArchVis Controls")
	FString MoveRightAxisName;

	/** Controls how aggressively mouse motion translates to character rotation in the pitch axis. */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "ArchVis Controls")
	float MouseSensitivityScale_Pitch;

	/** Controls how aggressively mouse motion translates to character rotation in the yaw axis. */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "ArchVis Controls")
	float MouseSensitivityScale_Yaw;

protected:
	
	// helper
	UE_API class UArchVisCharMovementComponent* GetArchVisCharMoveComponent() const;

	UE_API virtual void SetupPlayerInputComponent(UInputComponent* InputComponent) override;
	UE_API virtual FRotator GetViewRotation() const override;

	// input handlers
	UE_API virtual void Turn(float Val);
	UE_API virtual void TurnAtRate(float Val);
	UE_API virtual void LookUp(float Val);
	UE_API virtual void LookUpAtRate(float Val);
	UE_API virtual void MoveRight(float Val);
	UE_API virtual void MoveForward(float Val);

};

#undef UE_API
